// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;

public class PlainPropsTests : TestModuleRules
{
	static PlainPropsTests()
	{
		if (InTestMode)
		{
			TestMetadata = new Metadata();
			TestMetadata.TestName = "PlainProps";
			TestMetadata.TestShortName = "PlainProps";
			TestMetadata.ReportType = "testdata";
			TestMetadata.Deactivated = true;
			//TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.Linux);
			//TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.Mac);

			string PlatformCompilationArgs;
			foreach (var Platform in UnrealTargetPlatform.GetValidPlatforms())
			{
				PlatformCompilationArgs = "-allmodules";
				TestMetadata.PlatformCompilationExtraArgs.Add(Platform, PlatformCompilationArgs);
			}

			// Platform-specific tags
			//TestMetadata.PlatformTags.Add(UnrealTargetPlatform.Linux, "~[.]~[Slow]");
		}
	}
	public PlainPropsTests(ReadOnlyTargetRules Target) : base(Target, true)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",					// For Misc/ConfigCacheIni.h test setup
				"LowLevelTestsRunner",	// For TestCommon/Initialization.h test setup
				"PlainProps"
			});

		if (Target.bBuildWithEditorOnlyData)
		{
			PrivateDependencyModuleNames.AddRange(new string[] { "DesktopPlatform" });
		}
	}
}
