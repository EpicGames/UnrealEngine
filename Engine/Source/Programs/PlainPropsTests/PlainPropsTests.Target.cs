// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class PlainPropsTestsTarget : TestTargetRules
{
	public PlainPropsTestsTarget(TargetInfo Target) : base(Target)
	{
		// Collects all tests decorated with #if WITH_LOW_LEVELTESTS from dependencies
		bWithLowLevelTestsOverride = true;
		bCompileWithPluginSupport = true;
		bCompileAgainstCoreUObject = false;

		// Build tests with different permuations of FName case preserving and number outlining
		// BuildEnvironment = TargetBuildEnvironment.Unique;
		// bFNameOutlineNumber = true;
		// GlobalDefinitions.Add("WITH_CASE_PRESERVING_NAME=0");

		bBuildWithEditorOnlyData = Target.Platform.IsInGroup(UnrealPlatformGroup.Desktop)
			&& (Target.Configuration == UnrealTargetConfiguration.Debug || Target.Configuration == UnrealTargetConfiguration.Development);
	}
}
