// Copyright Epic Games, Inc. All Rights Reserved.
#include "TestCommon/Initialization.h"
#include "Misc/ConfigCacheIni.h"

#include <catch2/catch_test_macros.hpp>

GROUP_BEFORE_GLOBAL(Catch::DefaultGroup)
{
	GConfig = new FConfigCacheIni(EConfigCacheType::Temporary); // for URLRequestFilterTest
	InitAll(/* bAllowLogging*/ true, /* bMultithreaded*/ true);
}

GROUP_AFTER_GLOBAL(Catch::DefaultGroup)
{
	CleanupAll();
	delete GConfig;
	GConfig = nullptr;
}
